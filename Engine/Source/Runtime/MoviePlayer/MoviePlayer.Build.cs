// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MoviePlayer : ModuleRules
{
	public MoviePlayer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
					"Engine",
					"ApplicationCore",
				}
		);

		PrivateDependencyModuleNames.AddRange(
            new string[] {
                    "Core",
                    "InputCore",
                    "RenderCore",
                    "CoreUObject",
					"HTTP",
					"MoviePlayerProxy",
                    "RHI",
                    "Slate",
					"SlateCore",
                    "HeadMountedDisplay"
				}
        );
	}
}
