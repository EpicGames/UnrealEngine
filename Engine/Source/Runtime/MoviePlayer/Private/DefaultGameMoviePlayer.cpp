// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultGameMoviePlayer.h"
#include "HAL/PlatformSplash.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreDelegates.h"
#include "EngineGlobals.h"
#include "Widgets/SViewport.h"
#include "Engine/GameEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/HittestGrid.h"
#include "Widgets/Layout/SBox.h"
#include "GlobalShader.h"
#include "MoviePlayerProxy.h"
#include "MoviePlayerThreading.h"
#include "MoviePlayerSettings.h"
#include "ShaderCompiler.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "IXRLoadingScreen.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/FileManager.h"
#include "Widgets/SVirtualWindow.h"
#include "Rendering/SlateDrawBuffer.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Http.h"
#include "HttpManager.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Engine/UserInterfaceSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogMoviePlayer, Log, All);

class SDefaultMovieBorder : public SBorder
{
public:

	SLATE_BEGIN_ARGS(SDefaultMovieBorder)
		: _OnKeyDown()
	{}

		SLATE_EVENT(FPointerEventHandler, OnMouseButtonDown)
		SLATE_EVENT(FOnKeyDown, OnKeyDown)
		SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	void Construct(const FArguments& InArgs)
	{
		OnKeyDownHandler = InArgs._OnKeyDown;

		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FCoreStyle::Get().GetBrush(TEXT("BlackBrush")))
			.OnMouseButtonDown(InArgs._OnMouseButtonDown)
			.Padding(0)[InArgs._Content.Widget]);

	}

	/**
	* Set the handler to be invoked when the user presses a key.
	*
	* @param InHandler   Method to execute when the user presses a key
	*/
	void SetOnOnKeyDown(const FOnKeyDown& InHandler)
	{
		OnKeyDownHandler = InHandler;
	}

	/**
	* Overrides SWidget::OnKeyDown()
	* executes OnKeyDownHandler if it is bound
	*/
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (OnKeyDownHandler.IsBound())
		{
			// If a handler is assigned, call it.
			return OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
		}
		return SBorder::OnKeyDown(MyGeometry, InKeyEvent);
	}

	/**
	* Overrides SWidget::SupportsKeyboardFocus()
	* Must support keyboard focus to accept OnKeyDown events
	*/
	bool SupportsKeyboardFocus() const override
	{
		return true;
	}

protected:

	FOnKeyDown OnKeyDownHandler;
};

TSharedPtr<FDefaultGameMoviePlayer> FDefaultGameMoviePlayer::MoviePlayer;

FDefaultGameMoviePlayer* FDefaultGameMoviePlayer::Get()
{
	return MoviePlayer.Get();
}

FDefaultGameMoviePlayer::FDefaultGameMoviePlayer()
	: FTickableObjectRenderThread(false, true)
	, SyncMechanism(NULL)
	, MovieStreamingIsDone(1)
	, LoadingIsDone(1)
	, IsMoviePlaying(false)
	, bUserCalledFinish(false)
	, bMainWindowClosed(false)
	, LoadingScreenAttributes()
	, LastPlayTime(0.0)
	, bInitialized(false)
	, bIsPlayOnBlockingEnabled(false)
	, bIsSlateThreadAllowed(true)
	, ViewportDPIScale(1.0f)
	, BlockingRefCount(0)
	, LastBlockingTickTime(0.0)
{
	FCoreDelegates::IsLoadingMovieCurrentlyPlaying.BindRaw(this, &FDefaultGameMoviePlayer::IsMovieCurrentlyPlaying);
    FCoreDelegates::RegisterMovieStreamerDelegate.AddRaw(this, &FDefaultGameMoviePlayer::RegisterMovieStreamer);
}

FDefaultGameMoviePlayer::~FDefaultGameMoviePlayer()
{
	if ( bInitialized )
	{
		// This should not happen if initialize was called correctly.  This is a fallback to ensure that the movie player rendering tickable gets unregistered on the rendering thread correctly
		Shutdown();
	}
	else if (GIsRHIInitialized)
	{
		// Even when uninitialized we must safely unregister the movie player on the render thread
		FDefaultGameMoviePlayer* InMoviePlayer = this;
		ENQUEUE_RENDER_COMMAND(UnregisterMoviePlayerTickable)(
			[InMoviePlayer](FRHICommandListImmediate& RHICmdList)
			{
				InMoviePlayer->Unregister();
			});
	}

	FCoreDelegates::IsLoadingMovieCurrentlyPlaying.Unbind();

	FlushRenderingCommands();
}

void FDefaultGameMoviePlayer::RegisterMovieStreamer(TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer)
{
	if (InMovieStreamer.IsValid() && !MovieStreamers.Contains(InMovieStreamer))
	{
		MovieStreamers.Add(InMovieStreamer);
		InMovieStreamer->OnCurrentMovieClipFinished().AddRaw(this, &FDefaultGameMoviePlayer::BroadcastMovieClipFinished);
	}
}

void FDefaultGameMoviePlayer::Initialize(FSlateRenderer& InSlateRenderer, TSharedPtr<SWindow> TargetRenderWindow)
{
	if(bInitialized)
	{
		return;
	}

	LLM_SCOPE_BYNAME(TEXT("SlateMoviePlayer"));
	UE_LOGF(LogMoviePlayer, Log, "Initializing movie player");

	FDefaultGameMoviePlayer* InMoviePlayer = this;
	ENQUEUE_RENDER_COMMAND(RegisterMoviePlayerTickable)(
		[InMoviePlayer](FRHICommandListImmediate& RHICmdList)
		{
			InMoviePlayer->Register();
		});

	bInitialized = true;

	// Initialize shaders, because otherwise they might not be guaranteed to exist at this point
	if (!FPlatformProperties::RequiresCookedData())
	{
		TArray<int32> ShaderMapIds;
		ShaderMapIds.Add(GlobalShaderMapId);
		GShaderCompilingManager->FinishCompilation(TEXT("Global"), ShaderMapIds);
	}

	// Add a delegate to start playing movies when we start loading a map
	FCoreUObjectDelegates::PreLoadMap.AddRaw( this, &FDefaultGameMoviePlayer::OnPreLoadMap );

	// Shutdown the movie player if the app is exiting
	FCoreDelegates::OnPreExit.AddRaw( this, &FDefaultGameMoviePlayer::Shutdown );

	FPlatformSplash::Hide();

    // Use the passed in RenderWindow if it was provided, create one otherwise
    const TSharedRef<SWindow> GameWindow = TargetRenderWindow.IsValid() ? TargetRenderWindow.ToSharedRef() : UGameEngine::CreateGameWindow();

	TSharedPtr<SViewport> MovieViewport;

	VirtualRenderWindow =
		SNew(SVirtualWindow)
		.Size(GameWindow->GetClientSizeInScreen());

	WidgetRenderer = MakeShared<FMoviePlayerWidgetRenderer, ESPMode::ThreadSafe>(GameWindow, VirtualRenderWindow, &InSlateRenderer);

	LoadingScreenContents = SNew(SDefaultMovieBorder)
		.OnKeyDown(this, &FDefaultGameMoviePlayer::OnLoadingScreenKeyDown)
		.OnMouseButtonDown(this, &FDefaultGameMoviePlayer::OnLoadingScreenMouseButtonDown)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(this, &FDefaultGameMoviePlayer::GetMovieWidth)
				.HeightOverride(this, &FDefaultGameMoviePlayer::GetMovieHeight)
				[
					SAssignNew(MovieViewport, SViewport)
					.EnableGammaCorrection(false)
					.Visibility(this, &FDefaultGameMoviePlayer::GetViewportVisibility)
				]
			]
			+SOverlay::Slot()
			[
				SAssignNew(UserWidgetDPIScaler, SDPIScaler)
				[
				SAssignNew(UserWidgetHolder, SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush(TEXT("NoBorder")))
				.Padding(0)
			]
			]
		];

	MovieViewportWeakPtr = MovieViewport;
	MovieViewport->SetActive(true);

	MainWindow = GameWindow;

	GameWindow->GetOnWindowClosedEvent().AddRaw(this, &FDefaultGameMoviePlayer::OnMainWindowClosed);
}

void FDefaultGameMoviePlayer::OnMainWindowClosed(const TSharedRef<SWindow>& Window)
{
	bMainWindowClosed = true;
}

void FDefaultGameMoviePlayer::Shutdown()
{
	UE_LOGF(LogMoviePlayer, Log, "Shutting down movie player");

	FMoviePlayerProxy::UnregisterServer();
	TSharedPtr<SWindow> MainWindowShared = MainWindow.Pin();
	if (MainWindowShared.IsValid())
	{
		MainWindowShared->GetOnWindowClosedEvent().RemoveAll(this);
		MainWindowShared.Reset();
	}

	StopMovie();
	WaitForMovieToFinish();

	FDefaultGameMoviePlayer* InMoviePlayer = this;
	ENQUEUE_RENDER_COMMAND(UnregisterMoviePlayerTickable)(
		[InMoviePlayer](FRHICommandListImmediate& RHICmdList)
		{
			InMoviePlayer->Unregister();
		});

	bInitialized = false;

	FCoreDelegates::OnPreExit.RemoveAll(this);
	FCoreUObjectDelegates::PreLoadMap.RemoveAll( this );
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	LoadingScreenContents.Reset();
	UserWidgetHolder.Reset();
	MainWindow.Reset();
	VirtualRenderWindow.Reset();

	MovieStreamers.Empty();
	ActiveMovieStreamer.Reset();

	LoadingScreenAttributes = FLoadingScreenAttributes();

	if( SyncMechanism )
	{
		SyncMechanism->DestroySlateThread();
		FScopeLock SyncMechanismLock(&SyncMechanismCriticalSection);
		delete SyncMechanism;
		SyncMechanism = NULL;
	}
}
void FDefaultGameMoviePlayer::PassLoadingScreenWindowBackToGame() const
{
	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (MainWindow.IsValid() && GameEngine)
	{
		GameEngine->GameViewportWindow = MainWindow;
	}
	else
	{
		UE_LOGF(LogMoviePlayer, Warning, "PassLoadingScreenWindowBackToGame failed.  No Window" );
	}
}

void FDefaultGameMoviePlayer::SetupLoadingScreen(const FLoadingScreenAttributes& InLoadingScreenAttributes)
{
	if (!CanPlayMovie())
	{
		LoadingScreenAttributes = FLoadingScreenAttributes();
		UE_LOGF(LogMoviePlayer, Warning, "Initial loading screen disabled from BaseDeviceProfiles.ini: r.AndroidDisableThreadedRenderingFirstLoad=1");
	}
	else
	{
	    LoadingScreenAttributes = InLoadingScreenAttributes;
    }
}

bool FDefaultGameMoviePlayer::HasEarlyStartupMovie() const
{
#if PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK
	return LoadingScreenAttributes.bAllowInEarlyStartup == true;
#else
	return false;
#endif
}

bool FDefaultGameMoviePlayer::PlayEarlyStartupMovies()
{
	if(HasEarlyStartupMovie())
	{
		return PlayMovie();
	}

	return false;
}

bool FDefaultGameMoviePlayer::PlayMovie()
{
	bool bBeganPlaying = false;

	// Allow systems to hook onto the movie player and provide loading screen data on demand
	// if it has not been setup explicitly by the user.
	if ( !LoadingScreenIsPrepared() )
	{
		OnPrepareLoadingScreenDelegate.Broadcast();
	}

	if (LoadingScreenIsPrepared() && !IsMovieCurrentlyPlaying() && FPlatformMisc::NumberOfCores() > 1)
	{
		check(LoadingScreenAttributes.IsValid());
		bUserCalledFinish = false;

		LastPlayTime = FPlatformTime::Seconds();

		ActiveMovieStreamer.Reset();
		if (MovieStreamingIsPrepared())
		{
			for (TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> MovieStreamer : MovieStreamers)
			{
				if (MovieStreamer->Init(LoadingScreenAttributes.MoviePaths, LoadingScreenAttributes.PlaybackType))
				{
					ActiveMovieStreamer = MovieStreamer;
					if (MovieViewportWeakPtr.IsValid())
					{
						MovieViewportWeakPtr.Pin()->SetViewportInterface(MovieStreamer->GetViewportInterface().ToSharedRef());
					}
					break;
				}
			}
		}

		if (ActiveMovieStreamer.IsValid() || !MovieStreamingIsPrepared())
		{
			MovieStreamingIsDone.Set(MovieStreamingIsPrepared() ? 0 : 1);
			LoadingIsDone.Set(0);
			IsMoviePlaying = true;

			UserWidgetDPIScaler->SetDPIScale(GetViewportDPIScale());

			UserWidgetHolder->SetContent(LoadingScreenAttributes.WidgetLoadingScreen.IsValid() ? LoadingScreenAttributes.WidgetLoadingScreen.ToSharedRef() : SNullWidget::NullWidget);
			VirtualRenderWindow->Resize(MainWindow.Pin()->GetClientSizeInScreen());
			VirtualRenderWindow->SetContent(LoadingScreenContents.ToSharedRef());

			// Register the movie viewport so that it can receive user input.
			// There is only a valid viewport if we have a movie streamer as the streamer sets it.
			if ((!FPlatformProperties::SupportsWindowedMode()) && (ActiveMovieStreamer.IsValid()))
			{
				TSharedPtr<SViewport> MovieViewport = MovieViewportWeakPtr.Pin();
				if (MovieViewport.IsValid())
				{
					// Let the streamer know about the previous viewport interface.
					TSharedPtr<SViewport> GameViewport = FSlateApplication::Get().GetGameViewport();
					if (GameViewport.IsValid())
					{
						TSharedPtr<ISlateViewport> ViewportInterface = GameViewport->GetViewportInterface().Pin();
						ActiveMovieStreamer->PreviousViewportInterface(ViewportInterface);
					}

					FSlateApplication::Get().RegisterGameViewport(MovieViewport.ToSharedRef());
				}
			}

			{
				FScopeLock SyncMechanismLock(&SyncMechanismCriticalSection);
				SyncMechanism = new FSlateLoadingSynchronizationMechanism(WidgetRenderer, ActiveMovieStreamer);
				SyncMechanism->Initialize();
			}

			bBeganPlaying = true;
			LastBlockingTickTime = FPlatformTime::Seconds();
			OnAsyncLoadingFlushUpdateDelegateHandle = FCoreDelegates::OnAsyncLoadingFlushUpdate.AddRaw(this, &FDefaultGameMoviePlayer::BlockingTick);
		}

		//Allow anything that set up this LoadingScreenAttribute to know the loading screen is now displaying
		if (bBeganPlaying)
		{
			OnMoviePlaybackStarted().Broadcast();
		}
	}

	return bBeganPlaying;
}

/** Check if the device can render on a parallel thread on the initial loading*/
bool FDefaultGameMoviePlayer::CanPlayMovie() const
{
	const IConsoleVariable *const CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AndroidDisableThreadedRenderingFirstLoad"));
	if (CVar && CVar->GetInt() != 0)
	{
		return (GEngine && GEngine->IsInitialized());
	}
	return true;
}

void FDefaultGameMoviePlayer::StopMovie()
{
	LastPlayTime = 0;
	bUserCalledFinish = true;
}

void FDefaultGameMoviePlayer::WaitForMovieToFinish(bool bAllowEngineTick)
{
	const bool bEnforceMinimumTime = LoadingScreenAttributes.MinimumLoadingScreenDisplayTime >= 0.0f;

	if (LoadingScreenIsPrepared() && IsMovieCurrentlyPlaying())
	{

		if (SyncMechanism)
		{
			SyncMechanism->DestroySlateThread();

			FScopeLock SyncMechanismLock(&SyncMechanismCriticalSection);
			delete SyncMechanism;
			SyncMechanism = nullptr;
		}
		if( !bEnforceMinimumTime )
		{
			LoadingIsDone.Set(1);
		}

        if (TSharedPtr<SWindow> MainWindowPtr = MainWindow.Pin())
        {
            // Transfer the content to the main window
			MainWindowPtr->SetContent(LoadingScreenContents.ToSharedRef());
        }
        if (VirtualRenderWindow.IsValid())
        {
            VirtualRenderWindow->SetContent(SNullWidget::NullWidget);
        }

		const bool bAutoCompleteWhenLoadingCompletes = LoadingScreenAttributes.bAutoCompleteWhenLoadingCompletes;
		const bool bWaitForManualStop = LoadingScreenAttributes.bWaitForManualStop;

		FSlateApplication& SlateApp = FSlateApplication::Get();

		// Make sure the movie player widget has user focus to accept keypresses
		if (LoadingScreenContents.IsValid())
		{
			SlateApp.SetAllUserFocus(LoadingScreenContents);
		}

		// Continue to wait until the user calls finish (if enabled) or when loading completes or the minimum enforced time (if any) has been reached.
		// Don't continue playing on game shutdown
		while ( !IsEngineExitRequested() &&
				((bWaitForManualStop && !bUserCalledFinish)
			||	(!bUserCalledFinish && !bEnforceMinimumTime && !IsMovieStreamingFinished() && !bAutoCompleteWhenLoadingCompletes)
			||	(bEnforceMinimumTime && (FPlatformTime::Seconds() - LastPlayTime) < LoadingScreenAttributes.MinimumLoadingScreenDisplayTime)))
		{
			BeginExitIfRequested();

			// If we are in a loading loop, and this is the last movie in the playlist.. assume you can break out
			// unless waiting for manual stop that hasn't been signaled yet.
			if (ActiveMovieStreamer.IsValid() && LoadingScreenAttributes.PlaybackType == MT_LoadingLoop && ActiveMovieStreamer->IsLastMovieInPlaylist() && (!bWaitForManualStop || (bWaitForManualStop && bUserCalledFinish)))
			{
				break;
			}

			if (FSlateApplication::IsInitialized())
			{
				// Break out of the loop if the main window is closed during the movie.
				if ( !MainWindow.IsValid() || bMainWindowClosed.Load() )
				{
					if (ActiveMovieStreamer.IsValid())
					{
						ActiveMovieStreamer->ForceCompletion();
					}
					break;
				}

				FPlatformApplicationMisc::PumpMessages(true);

				FCoreDelegates::OnSamplingInput.Broadcast();

				SlateApp.PollGameDeviceState();
				// Gives widgets a chance to process any accumulated input
				SlateApp.FinishedInputThisFrame();

				float DeltaTime = SlateApp.GetDeltaTime();

				if (ActiveMovieStreamer.IsValid())
				{
					ActiveMovieStreamer->TickPreEngine();
				}

				if (GEngine && bAllowEngineTick && LoadingScreenAttributes.bAllowEngineTick)
				{
					GEngine->Tick(DeltaTime, false);
				}

				if (ActiveMovieStreamer.IsValid())
				{
					ActiveMovieStreamer->TickPostEngine();
				}

				GFrameCounter++;

				FDefaultGameMoviePlayer* InMoviePlayer = this;
				ENQUEUE_RENDER_COMMAND(BeginLoadingMovieFrameAndTickMovieStreamer)(
					[InMoviePlayer, DeltaTime, CurrentFrameCounter = GFrameCounter](FRHICommandListImmediate& RHICmdList)
					{
						GFrameCounterRenderThread = CurrentFrameCounter;
						GFrameNumberRenderThread++;
						InMoviePlayer->TickStreamer(RHICmdList, DeltaTime);
					}
				);

				{
					FSlateRenderer* SlateRenderer = SlateApp.GetRenderer();
					FScopeLock ScopeLock(SlateRenderer->GetResourceCriticalSection());

					SlateApp.Tick();

					// Synchronize the game thread and the render thread so that the render thread doesn't get too far behind.
					SlateRenderer->Sync();
				}

				ENQUEUE_RENDER_COMMAND(FinishLoadingMovieFrame)(
					[](FRHICommandListImmediate& RHICmdList)
					{
						RHICmdList.EndFrame();
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
					}
				);
				FlushRenderingCommands();

				if (ActiveMovieStreamer.IsValid())
				{
					ActiveMovieStreamer->TickPostRender();
				}
			}
		}

		if (UserWidgetHolder.IsValid())
		{
			UserWidgetHolder->SetContent(SNullWidget::NullWidget);
		}
		LoadingIsDone.Set(1);
		IsMoviePlaying = false;
		FCoreDelegates::OnAsyncLoadingFlushUpdate.Remove(OnAsyncLoadingFlushUpdateDelegateHandle);
		OnAsyncLoadingFlushUpdateDelegateHandle.Reset();

		IXRLoadingScreen* LoadingScreen;
		if (GEngine && GEngine->XRSystem.IsValid() && (LoadingScreen = GEngine->XRSystem->GetLoadingScreen()) != nullptr && SyncMechanism == nullptr)
		{
			LoadingScreen->ClearSplashes();
		}

		MovieStreamingIsDone.Set(1);

		FlushRenderingCommands();

		if( ActiveMovieStreamer.IsValid() )
		{
			ActiveMovieStreamer->ForceCompletion();
		}

		// Allow the movie streamer to clean up any resources it uses once there are no movies to play.
		if( ActiveMovieStreamer.IsValid() )
		{
			ActiveMovieStreamer->Cleanup();
		}

		// Finally, clear out the loading screen attributes, forcing users to always
		// explicitly set the loading screen they want (rather than have stale loading screens)
		LoadingScreenAttributes = FLoadingScreenAttributes();

		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);

		// Don't switch the window on game shutdown
		if (GameEngine && !IsEngineExitRequested())
		{
			GameEngine->SwitchGameWindowToUseGameViewport();
		}

		BroadcastMoviePlaybackFinished();
	}
	else
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);

		// Don't switch the window on game shutdown
		if (GameEngine && !IsEngineExitRequested())
		{
			GameEngine->SwitchGameWindowToUseGameViewport();
		}
	}

}

bool FDefaultGameMoviePlayer::IsLoadingFinished() const
{
	return LoadingIsDone.GetValue() != 0;
}

bool FDefaultGameMoviePlayer::IsMovieCurrentlyPlaying() const
{
	return SyncMechanism != NULL;
}

bool FDefaultGameMoviePlayer::IsMovieStreamingFinished() const
{
	return MovieStreamingIsDone.GetValue() != 0;
}

void FDefaultGameMoviePlayer::BlockingStarted()
{
	if ((bIsPlayOnBlockingEnabled) && (bIsSlateThreadAllowed))
	{
		UE_LOGF(LogMoviePlayer, Verbose, "BlockingStarted %d", BlockingRefCount);
		BlockingRefCount++;
		PlayMovie();
	}
}

void FDefaultGameMoviePlayer::BlockingTick()
{
	check(IsInGameThread());
	if (IsMovieCurrentlyPlaying())
	{
		// Has enough time passed since the last tick?
		double Time = FPlatformTime::Seconds();
		double DeltaTime = Time - LastBlockingTickTime;
		if (DeltaTime > 0.1f)
		{
			// Yes. Time for another tick.
			LastBlockingTickTime = Time;

			// Call HTTP manager.
			FHttpManager& HttpManager = FHttpModule::Get().GetHttpManager();
			HttpManager.Tick(0.0f);

			// Callbacks.
			OnMoviePlaybackTick().Broadcast((float)DeltaTime);

			UE_LOGF(LogMoviePlayer, VeryVerbose, "BlockingTick deltatime:%f", DeltaTime);
		}
	}
}

void FDefaultGameMoviePlayer::BlockingFinished()
{
	if (bIsPlayOnBlockingEnabled)
	{
		// Only call WaitForMovieToFinish if we are playing a movie,
		// as WaitForMovieToFinish has side effects if a movie is not playing,
		// and this can cause a hang.
		if (LoadingScreenIsPrepared() && IsMovieCurrentlyPlaying())
		{
			UE_LOGF(LogMoviePlayer, Verbose, "BlockingFinished. Refcount: %d.", BlockingRefCount);
			WaitForMovieToFinish();
		}

		BlockingRefCount = 0;
	}
}

void FDefaultGameMoviePlayer::SetIsSlateThreadAllowed(bool bInIsSlateThreadAllowed)
{
	if (bIsSlateThreadAllowed != bInIsSlateThreadAllowed)
	{
		bIsSlateThreadAllowed = bInIsSlateThreadAllowed;

		// Can we use the Slate thread?
		if (bIsSlateThreadAllowed == false)
		{
			// Nope. Make sure its no longer running.
			BlockingFinished();
		}
	}
}

void FDefaultGameMoviePlayer::Tick( FRHICommandListImmediate& RHICmdList, float DeltaTime )
{
	check(IsInRenderingThread());
	if (MainWindow.IsValid() && VirtualRenderWindow.IsValid() && !IsLoadingFinished() && GDynamicRHI && !GDynamicRHI->RHIIsRenderingSuspended())
	{
		FScopeLock SyncMechanismLock(&SyncMechanismCriticalSection);
		if(SyncMechanism)
		{
			if(SyncMechanism->IsSlateDrawPassEnqueued())
			{
				GFrameNumberRenderThread++;
				TickStreamer(RHICmdList, DeltaTime);
				SyncMechanism->ResetSlateDrawPassEnqueued();
				RHICmdList.EndFrame();
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
			}
		}
	}
}

void FDefaultGameMoviePlayer::TickStreamer(FRHICommandListImmediate& RHICmdList, float DeltaTime)
{
	if (MovieStreamingIsPrepared() && ActiveMovieStreamer.IsValid() && !IsMovieStreamingFinished())
	{
		const bool bMovieIsDone = ActiveMovieStreamer->Tick(RHICmdList, DeltaTime);
		if (bMovieIsDone)
		{
			MovieStreamingIsDone.Set(1);
		}

		// commenting loading screen currently as we don't support changing/adding/removing splash screens on the renderthread
		/*IXRLoadingScreen* LoadingScreen;
		if (GEngine && GEngine->XRSystem.IsValid() && (LoadingScreen = GEngine->XRSystem->GetLoadingScreen()) != nullptr)
		{
			FTextureRHIRef Movie2DTexture = ActiveMovieStreamer->GetTexture();
			LoadingScreen->ClearSplashes();
			if (Movie2DTexture.IsValid() && !bMovieIsDone)
			{
				IXRLoadingScreen::FSplashDesc Splash;
				Splash.Texture = (FRHITexture*)Movie2DTexture.GetReference();
				Splash.bIsDynamic = true;
				const FIntPoint TextureSize = Movie2DTexture->GetSizeXY();
				const float	InvAspectRatio = (TextureSize.X > 0) ? float(TextureSize.Y) / float(TextureSize.X) : 1.0f;

				Splash.bIgnoreAlpha = true;
				Splash.Transform = FTransform(FVector(5.0f, 0.0f, 1.0f));
				Splash.QuadSize = FVector2D(8.0f, 8.0f*InvAspectRatio);
				LoadingScreen->AddSplash(Splash);
			}
		}*/
	}
}

TStatId FDefaultGameMoviePlayer::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDefaultGameMoviePlayer, STATGROUP_Tickables);
}

bool FDefaultGameMoviePlayer::IsTickable() const
{
	return true;
}

bool FDefaultGameMoviePlayer::LoadingScreenIsPrepared() const
{
	return LoadingScreenAttributes.WidgetLoadingScreen.IsValid() || MovieStreamingIsPrepared();
}

void FDefaultGameMoviePlayer::SetupLoadingScreenFromIni()
{
	// We may have already setup a movie from a startup module
	if( !LoadingScreenAttributes.IsValid() )
	{
		// fill out the attributes
		FLoadingScreenAttributes LoadingScreen;

		bool bWaitForMoviesToComplete = false;
		// Note: this code is executed too early so we cannot access UMoviePlayerSettings because the configs for that object have not been loaded and coalesced .  Have to read directly from the configs instead
		GConfig->GetBool(TEXT("/Script/MoviePlayer.MoviePlayerSettings"), TEXT("bWaitForMoviesToComplete"), bWaitForMoviesToComplete, GGameIni);
		GConfig->GetBool(TEXT("/Script/MoviePlayer.MoviePlayerSettings"), TEXT("bMoviesAreSkippable"), LoadingScreen.bMoviesAreSkippable, GGameIni);

		LoadingScreen.bAutoCompleteWhenLoadingCompletes = !bWaitForMoviesToComplete;

		TArray<FString> StartupMovies;
		GConfig->GetArray(TEXT("/Script/MoviePlayer.MoviePlayerSettings"), TEXT("StartupMovies"), StartupMovies, GGameIni);

		if (StartupMovies.Num() == 0)
		{
			StartupMovies.Add(TEXT("Default_Startup"));
		}

		// double check that the movies exist
		// We dont know the extension so compare against any file in the directory with the same name for now
		// @todo New Movie Player: movies should have the extension on them when set via the project settings
		TArray<FString> ExistingMovieFiles;
		IFileManager::Get().FindFiles(ExistingMovieFiles, *(FPaths::ProjectContentDir() + TEXT("Movies")));

		bool bHasValidMovie = false;
		for(const FString& Movie : StartupMovies)
		{
			bool bFound = ExistingMovieFiles.ContainsByPredicate(
				[&Movie](const FString& ExistingMovie)
				{
					return ExistingMovie.Contains(Movie);
				});

			if(bFound)
			{
				bHasValidMovie = true;
				LoadingScreen.MoviePaths.Add(Movie);
			}
		}

		if(bHasValidMovie)
		{
			// These movies are all considered safe to play in very early startup sequences
			LoadingScreen.bAllowInEarlyStartup = true;

			// now setup the actual loading screen
			SetupLoadingScreen(LoadingScreen);
		}
	}
}

void FDefaultGameMoviePlayer::SetViewportDPIScale(float InViewportDPIScale)
{
	ViewportDPIScale = InViewportDPIScale;

	// Turn off scale in the renderer as we have our own scale.
	if (WidgetRenderer != nullptr)
	{
		WidgetRenderer->EnableDPIScale(false);
	}
}

bool FDefaultGameMoviePlayer::MovieStreamingIsPrepared() const
{
	return MovieStreamers.Num() > 0 && LoadingScreenAttributes.MoviePaths.Num() > 0;
}

FVector2D FDefaultGameMoviePlayer::GetMovieSize() const
{
	const FVector2D ScreenSize = MainWindow.Pin()->GetClientSizeInScreen();
	if (MovieStreamingIsPrepared() && ActiveMovieStreamer.IsValid())
	{
		const double MovieAspectRatio = ActiveMovieStreamer->GetAspectRatio();
		const double ScreenAspectRatio = ScreenSize.X / ScreenSize.Y;
		if (MovieAspectRatio < ScreenAspectRatio)
		{
			return FVector2D(ScreenSize.Y * MovieAspectRatio, ScreenSize.Y);
		}
		else
		{
			return FVector2D(ScreenSize.X, ScreenSize.X / MovieAspectRatio);
		}
	}

	// No movie, so simply return the size of the window
	return ScreenSize;
}

FOptionalSize FDefaultGameMoviePlayer::GetMovieWidth() const
{
	return (float)GetMovieSize().X;
}

FOptionalSize FDefaultGameMoviePlayer::GetMovieHeight() const
{
	return (float)GetMovieSize().Y;
}

EVisibility FDefaultGameMoviePlayer::GetSlateBackgroundVisibility() const
{
	return MovieStreamingIsPrepared() && ActiveMovieStreamer.IsValid() && !IsMovieStreamingFinished() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FDefaultGameMoviePlayer::GetViewportVisibility() const
{
	return MovieStreamingIsPrepared() && ActiveMovieStreamer.IsValid() && !IsMovieStreamingFinished() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FDefaultGameMoviePlayer::OnLoadingScreenMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	return OnAnyDown();
}

FReply FDefaultGameMoviePlayer::OnLoadingScreenKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
{
	return OnAnyDown();
}

FReply FDefaultGameMoviePlayer::OnAnyDown()
{
	if (IsLoadingFinished())
	{
		if (LoadingScreenAttributes.bMoviesAreSkippable)
		{
			MovieStreamingIsDone.Set(1);
			if (ActiveMovieStreamer.IsValid())
			{
				ActiveMovieStreamer->ForceCompletion();
			}
		}

		if (IsMovieStreamingFinished())
		{
			bUserCalledFinish = true;
		}
	}

	return FReply::Handled();
}

void FDefaultGameMoviePlayer::OnPreLoadMap(const FString& LevelName)
{
	if (bIsPlayOnBlockingEnabled == false)
	{
		UE_LOGF(LogMoviePlayer, Verbose, "PreLoadMap");
		FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

		if (PlayMovie())
		{
			FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FDefaultGameMoviePlayer::OnPostLoadMap);
		}
	}
}

void FDefaultGameMoviePlayer::OnPostLoadMap(UWorld* LoadedWorld)
{
	if (bIsPlayOnBlockingEnabled == false)
	{
		UE_LOGF(LogMoviePlayer, Verbose, "PostLoadMap");
		if (!LoadingScreenAttributes.bAllowEngineTick)
		{
			// If engine tick is enabled, we don't want to tick here and instead want to run from the WaitForMovieToFinish call in LaunchEngineLoop
			WaitForMovieToFinish();
		}
	}
}

void FDefaultGameMoviePlayer::SetSlateOverlayWidget(TSharedPtr<SWidget> NewOverlayWidget)
{
	if (ActiveMovieStreamer.IsValid() && UserWidgetHolder.IsValid())
	{
		UserWidgetHolder->SetContent(NewOverlayWidget.ToSharedRef());
	}
}

bool FDefaultGameMoviePlayer::WillAutoCompleteWhenLoadFinishes()
{
	return LoadingScreenAttributes.bAutoCompleteWhenLoadingCompletes || (LoadingScreenAttributes.PlaybackType == MT_LoadingLoop && (ActiveMovieStreamer.IsValid() && ActiveMovieStreamer->IsLastMovieInPlaylist()));
}

FString FDefaultGameMoviePlayer::GetMovieName()
{
	return ActiveMovieStreamer.IsValid() ? ActiveMovieStreamer->GetMovieName() : TEXT("");
}

bool FDefaultGameMoviePlayer::IsLastMovieInPlaylist()
{
	return ActiveMovieStreamer.IsValid() ? ActiveMovieStreamer->IsLastMovieInPlaylist() : false;
}

FMoviePlayerWidgetRenderer::FMoviePlayerWidgetRenderer(TSharedPtr<SWindow> InMainWindow, TSharedPtr<SVirtualWindow> InVirtualRenderWindow, FSlateRenderer* InRenderer)
	: MainWindow(InMainWindow.Get())
	, VirtualRenderWindow(InVirtualRenderWindow.ToSharedRef())
	, SlateRenderer(InRenderer)
	, bIsDPIScaleEnabled(true)
{
	HittestGrid = MakeShareable(new FHittestGrid);
}

void FMoviePlayerWidgetRenderer::EnableDPIScale(bool bShouldEnable)
{
	bIsDPIScaleEnabled = bShouldEnable;
}

void FMoviePlayerWidgetRenderer::DrawWindow(float DeltaTime)
{
	if (GDynamicRHI && GDynamicRHI->RHIIsRenderingSuspended())
	{
		// This avoids crashes if we Suspend rendering whilst the loading screen is up
		// as we don't want Slate to submit any more draw calls until we Resume.
		return;
	}
	float Scale = FSlateApplication::Get().GetApplicationScale();
	if (bIsDPIScaleEnabled)
	{
		Scale *= MainWindow->GetDPIScaleFactor();
	}

	FVector2D DrawSize = VirtualRenderWindow->GetClientSizeInScreen() / Scale;

	FSlateApplication::Get().Tick(ESlateTickType::Time);

	FGeometry WindowGeometry = FGeometry::MakeRoot(DrawSize, FSlateLayoutTransform(Scale));

	VirtualRenderWindow->SlatePrepass(WindowGeometry.Scale);

	FSlateRect ClipRect = WindowGeometry.GetLayoutBoundingRect();

	HittestGrid->SetHittestArea(VirtualRenderWindow->GetPositionInScreen(), VirtualRenderWindow->GetViewportSize());
	HittestGrid->Clear();

	{
		// Get the free buffer & add our virtual window
		FSlateRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *SlateRenderer };
		FSlateWindowElementList& WindowElementList = ScopedDrawBuffer.GetDrawBuffer().AddWindowElementList(VirtualRenderWindow);

		WindowElementList.SetRenderTargetWindow(MainWindow);

		int32 MaxLayerId = 0;
		{
			FPaintArgs PaintArgs(nullptr, *HittestGrid, FVector2D::ZeroVector, FSlateApplication::Get().GetCurrentTime(), FSlateApplication::Get().GetDeltaTime());

			// Paint the window
			MaxLayerId = VirtualRenderWindow->Paint(
				PaintArgs,
				WindowGeometry, ClipRect,
				WindowElementList,
				0,
				FWidgetStyle(),
				VirtualRenderWindow->IsEnabled());
		}

		SlateRenderer->DrawWindows(ScopedDrawBuffer.GetDrawBuffer());
		ScopedDrawBuffer.GetDrawBuffer().ViewOffset = FVector2D::ZeroVector;
	}
}

float FDefaultGameMoviePlayer::GetViewportDPIScale() const
{
	return ViewportDPIScale;
}
void FDefaultGameMoviePlayer::ForceCompletion()
{
	bUserCalledFinish = true;
	MovieStreamingIsDone.Set(1);

	if (ActiveMovieStreamer.IsValid())
	{
		ActiveMovieStreamer->ForceCompletion();
	}
}
/*Interrupts*/
void FDefaultGameMoviePlayer::Suspend()
{
	if (ActiveMovieStreamer.IsValid())
	{
		ActiveMovieStreamer->Suspend();
	}
}

void FDefaultGameMoviePlayer::Resume()
{
	if (ActiveMovieStreamer.IsValid())
	{
		ActiveMovieStreamer->Resume();
	}
}

void FDefaultGameMoviePlayer::SetIsPlayOnBlockingEnabled(bool bIsEnabled)
{
	if (bIsPlayOnBlockingEnabled != bIsEnabled)
	{
		bIsPlayOnBlockingEnabled = bIsEnabled;

		if (bIsPlayOnBlockingEnabled)
		{
			FMoviePlayerProxy::RegisterServer(this);
		}
		else
		{
			BlockingFinished();
			FMoviePlayerProxy::UnregisterServer();
		}
	}
}

