// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "Types/SlateStructs.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "MoviePlayer.h"
#include "MoviePlayerProxyServer.h"
#include "TickableObjectRenderThread.h"

#include "Misc/CoreDelegates.h"

class FWidgetRenderer;
class SVirtualWindow;

class FMoviePlayerWidgetRenderer
{
public:
	FMoviePlayerWidgetRenderer(TSharedPtr<SWindow> InMainWindow, TSharedPtr<SVirtualWindow> InVirtualRenderWindowWindow, FSlateRenderer* InRenderer);

	/**
	 * Call this enable or disable DPI scale. *
	 *
	 * @param bInIsEanbled True to apply a DPI scale.
	 */
	void EnableDPIScale(bool bShouldEnable);
	void DrawWindow(float DeltaTime);

private:
	/** The actual window content will be drawn to */
	/** Note: This is raw as we SWindows registered with SlateApplication are not thread safe */
	SWindow* MainWindow;

	/** Virtual window that we render to instead of the main slate window (for thread safety).  Shares only the same backbuffer as the main window */
	TSharedRef<class SVirtualWindow> VirtualRenderWindow;

	TSharedPtr<FHittestGrid> HittestGrid;

	FSlateRenderer* SlateRenderer;

	FViewportRHIRef ViewportRHI;

	/** If true then apply a DPI scale. */
	bool bIsDPIScaleEnabled;
};

/** An implementation of the movie player/loading screen we will use */
class FDefaultGameMoviePlayer : public FTickableObjectRenderThread, public IGameMoviePlayer,
	public IMoviePlayerProxyServer, public TSharedFromThis<FDefaultGameMoviePlayer>
{
public:
	static void Create()
	{
		check(IsInGameThread() && !IsInSlateThread());
		check(!MoviePlayer.IsValid());

		MoviePlayer = MakeShareable(new FDefaultGameMoviePlayer);
	}

	static void Destroy()
	{
		check(IsInGameThread() && !IsInSlateThread());
		MoviePlayer.Reset();
	}

	static FDefaultGameMoviePlayer* Get();
	~FDefaultGameMoviePlayer();

	/** IGameMoviePlayer Interface */
	virtual void RegisterMovieStreamer(TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer) override;
	virtual void Initialize(FSlateRenderer& InSlateRenderer, TSharedPtr<SWindow> TargetRenderWindow = nullptr) override;
	virtual void Shutdown() override;
	virtual void PassLoadingScreenWindowBackToGame() const override;
	virtual void SetupLoadingScreen(const FLoadingScreenAttributes& LoadingScreenAttributes) override;
	virtual bool HasEarlyStartupMovie() const override;
	virtual bool PlayEarlyStartupMovies() override;
	virtual bool PlayMovie() override;
	virtual void StopMovie() override;
	virtual void WaitForMovieToFinish(bool bAllowEngineTick = false) override;
	virtual bool IsLoadingFinished() const override;
	virtual bool IsMovieCurrentlyPlaying() const override;
	virtual bool LoadingScreenIsPrepared() const override;
	virtual void SetupLoadingScreenFromIni() override;
	virtual void SetViewportDPIScale(float InViewportDPIScale) override;
	
	bool IsInitialized() const override { return bInitialized; }

	/** Check if the initial movie(s) is still playing */
	virtual bool IsStartupMoviePlaying() const override { return IsMoviePlaying; };

	virtual FOnPrepareLoadingScreen& OnPrepareLoadingScreen() override { return OnPrepareLoadingScreenDelegate; }
	virtual FOnMoviePlaybackStarted& OnMoviePlaybackStarted() override { return OnMoviePlaybackStartedDelegate; }
	virtual FOnMoviePlaybackTick& OnMoviePlaybackTick() override { return OnMoviePlaybackTickDelegate; }
	virtual FOnMoviePlaybackFinished& OnMoviePlaybackFinished() override { return OnMoviePlaybackFinishedDelegate; }
	virtual FOnMovieClipFinished& OnMovieClipFinished() override { return OnMovieClipFinishedDelegate; }
	
	/** IMoviePlayerProxyServer interface. */
	virtual void BlockingStarted() override;
	virtual void BlockingTick() override;
	virtual void BlockingFinished() override;
	virtual void SetIsSlateThreadAllowed(bool bInIsSlateThreadAllowed) override;

	/** FTickableObjectRenderThread interface */
	virtual void Tick( FRHICommandListImmediate& RHICmdList, float DeltaTime ) override;
	virtual TStatId GetStatId() const override;
	virtual bool IsTickable() const override;
	
	/** Callback for clicking on the viewport */
	FReply OnLoadingScreenMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent);
	FReply OnLoadingScreenKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent);

	virtual void SetSlateOverlayWidget(TSharedPtr<SWidget> NewOverlayWidget) override;

	virtual bool WillAutoCompleteWhenLoadFinishes() override;

	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;

	virtual void ForceCompletion() override;
	virtual void Suspend() override;
	virtual void Resume() override;

	virtual void SetIsPlayOnBlockingEnabled(bool bIsEnabled) override;

	float GetViewportDPIScale() const;

	void OnMainWindowClosed(const TSharedRef<SWindow>& Window);

private:

	/** Ticks the underlying MovieStreamer.  Must be done exactly once before each DrawWindows call. */
	void TickStreamer(FRHICommandListImmediate& RHICmdList, float DeltaTime);

	/** True if we have both a registered movie streamer and movies to stream */
	bool MovieStreamingIsPrepared() const;

	/** True if movie streamer has finished streaming all the movies it wanted to */
	bool IsMovieStreamingFinished() const;

	/** Callbacks for movie sizing for the movie viewport */
	FVector2D GetMovieSize() const;
	FOptionalSize GetMovieWidth() const;
	FOptionalSize GetMovieHeight() const;
	EVisibility GetSlateBackgroundVisibility() const;
	EVisibility GetViewportVisibility() const;	
	
	/** Called via a delegate in the engine when maps start to load */
	void OnPreLoadMap(const FString& LevelName);

	/** Called via a delegate in the engine when maps finish loading */
	void OnPostLoadMap(UWorld* LoadedWorld);

	/** Check if the device can render on a parallel thread on the initial loading*/
	bool CanPlayMovie() const;
private:
	FDefaultGameMoviePlayer();

	FReply OnAnyDown();
	
	/** The movie streaming systems that will be used by us */
	TArray<TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe>> MovieStreamers;
	TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> ActiveMovieStreamer;

	/** The window that the loading screen resides in */
	TWeakPtr<class SWindow> MainWindow;
	/** The widget which includes all contents of the loading screen, widgets and movie player and all */
	TSharedPtr<class SWidget> LoadingScreenContents;
	/** The widget which holds the loading screen widget passed in via the FLoadingScreenAttributes */
	TSharedPtr<class SBorder> UserWidgetHolder;
	/** Virtual window that we render to instead of the main slate window (for thread safety).  Shares only the same backbuffer as the main window */
	TSharedPtr<class SVirtualWindow> VirtualRenderWindow;
	/** Viewport responsible for displaying the movie player render target */
	TWeakPtr<class SViewport> MovieViewportWeakPtr;

	/** The threading mechanism with which we handle running slate on another thread */
	class FSlateLoadingSynchronizationMechanism* SyncMechanism;

	/** True if all movies have successfully streamed and completed */
	FThreadSafeCounter MovieStreamingIsDone;

	/** True if the game thread has finished loading */
	FThreadSafeCounter LoadingIsDone;

	/** True if the movie player is not yet finished */
	bool IsMoviePlaying;

	/** User has called finish (needed if LoadingScreenAttributes.bAutoCompleteWhenLoadingCompletes is off) */
	bool bUserCalledFinish;
	
	/** Main window has closed, stop movie playback */
	TAtomic<bool> bMainWindowClosed;

	/** Attributes of the loading screen we are currently displaying */
	FLoadingScreenAttributes LoadingScreenAttributes;

	/** Called before a movie is queued up to play to configure the movie player accordingly. */
	FOnPrepareLoadingScreen OnPrepareLoadingScreenDelegate;
	
	FOnMoviePlaybackStarted OnMoviePlaybackStartedDelegate;

	/** Called periodically when the game thread is blocked and a movie is playing. */
	FOnMoviePlaybackTick OnMoviePlaybackTickDelegate;

	FOnMoviePlaybackFinished OnMoviePlaybackFinishedDelegate;

	FOnMovieClipFinished OnMovieClipFinishedDelegate;

	/** The last time a movie was started */
	double LastPlayTime;

	/** True if the movie player has been initialized */
	bool bInitialized;

	/** If true then play movie when blocking starts, if false then play movie on loadmap. */
	bool bIsPlayOnBlockingEnabled;
	/** If true then we can use the Slate thread. */
	bool bIsSlateThreadAllowed;

	/** Critical section to allow the slate loading thread and the render thread to safely utilize the synchronization mechanism for ticking Slate. */
	FCriticalSection SyncMechanismCriticalSection;

	/** Widget renderer used to tick and paint windows in a thread safe way */
	TSharedPtr<FMoviePlayerWidgetRenderer, ESPMode::ThreadSafe> WidgetRenderer;

	/** DPIScaler parented to the UserWidgetHolder to ensure correct scaling */
	TSharedPtr<class SDPIScaler> UserWidgetDPIScaler;

	/** Scale used by UserWidgetDPIScaler. */
	float ViewportDPIScale;

	/** Counts blocking starts - blocking finishes. */
	int32 BlockingRefCount;
	/** Last time we were able to tick during blocking. */
	double LastBlockingTickTime;
	/** Handle to our async loading update function. */
	FDelegateHandle OnAsyncLoadingFlushUpdateDelegateHandle;

private:
	/** Singleton handle */
	static TSharedPtr<FDefaultGameMoviePlayer> MoviePlayer;
};
