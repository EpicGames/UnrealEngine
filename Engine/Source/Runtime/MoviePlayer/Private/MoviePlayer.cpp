// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePlayer.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "NullMoviePlayer.h"
#include "DefaultGameMoviePlayer.h"
#include "Widgets/Images/SThrobber.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePlayer)

IMPLEMENT_MODULE(FDefaultModuleImpl, MoviePlayer);


/** A very simple loading screen sample test widget to use */
class SLoadingScreenTestWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLoadingScreenTestWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SThrobber)
				.Visibility(this, &SLoadingScreenTestWidget::GetLoadIndicatorVisibility)
			]
			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("MoviePlayerTestLoadingScreen", "LoadingComplete", "Loading complete!"))
				.Visibility(this, &SLoadingScreenTestWidget::GetMessageIndicatorVisibility)
			]
		];
	}

private:
	EVisibility GetLoadIndicatorVisibility() const
	{
		return GetMoviePlayer()->IsLoadingFinished() ? EVisibility::Collapsed : EVisibility::Visible;
	}
	
	EVisibility GetMessageIndicatorVisibility() const
	{
		return GetMoviePlayer()->IsLoadingFinished() ? EVisibility::Visible : EVisibility::Collapsed;
	}
};


bool FLoadingScreenAttributes::IsValid() const {return WidgetLoadingScreen.IsValid() || MoviePaths.Num() > 0;}

TSharedRef<SWidget> FLoadingScreenAttributes::NewTestLoadingScreenWidget()
{
	return SNew(SLoadingScreenTestWidget);
}

void CreateMoviePlayer()
{
	// Do not create the movie player if it already exists
	if(!GetMoviePlayer())
	{
		if (!IsMoviePlayerEnabled() || GUsingNullRHI)
		{
			return FNullGameMoviePlayer::Create();
		}
		else
		{
			return FDefaultGameMoviePlayer::Create();
		}
	}
}

void DestroyMoviePlayer()
{
	IGameMoviePlayer* MoviePlayer = GetMoviePlayer();

	if(MoviePlayer)
	{

		if (!IsMoviePlayerEnabled() || GUsingNullRHI)
		{
			return FNullGameMoviePlayer::Destroy();
		}
		else
		{
			return FDefaultGameMoviePlayer::Destroy();
		}
	}
}


IGameMoviePlayer* GetMoviePlayer()
{
	if (!FCommandLine::IsInitialized() || !IsMoviePlayerEnabled() || GUsingNullRHI)
	{
		return FNullGameMoviePlayer::Get();
	}
	else
	{
		return FDefaultGameMoviePlayer::Get();
	}
}

IGameMoviePlayer& GetMoviePlayerRef()
{
	return *GetMoviePlayer();
}

bool IsMoviePlayerEnabled()
{
	bool bEnabled = !GIsEditor && !IsRunningDedicatedServer() && !IsRunningCommandlet() && GUseThreadedRendering;

#if !UE_BUILD_SHIPPING
	static bool bNoLoadingScreen = FParse::Param(FCommandLine::Get(), TEXT("NoLoadingScreen"));
	bEnabled &= !bNoLoadingScreen;
#endif

	return bEnabled;
}

