// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePlayerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePlayerSettings)


#define LOCTEXT_NAMESPACE "MoviePlayerSettings"

UMoviePlayerSettings::UMoviePlayerSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
}

#undef LOCTEXT_NAMESPACE

