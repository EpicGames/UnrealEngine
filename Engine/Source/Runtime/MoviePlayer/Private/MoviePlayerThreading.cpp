// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePlayerThreading.h"
#include "MoviePlayer.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Framework/Application/SlateApplication.h"
#include "DefaultGameMoviePlayer.h"
#include "HAL/PlatformApplicationMisc.h"

FThreadSafeCounter FSlateLoadingSynchronizationMechanism::LoadingThreadInstanceCounter;

/**
 * The Slate thread is simply run on a worker thread.
 * Slate is run on another thread because the game thread (where Slate is usually run)
 * is blocked loading things. Slate is very modular, which makes it very easy to run on another
 * thread with no adverse effects.
 * It does not enqueue render commands, because the RHI is not thread safe. Thus, it waits to
 * enqueue render commands until the render thread tickables ticks, and then it calls them there.
 */
class FSlateLoadingThreadTask : public FRunnable
{
public:
	FSlateLoadingThreadTask(class FSlateLoadingSynchronizationMechanism& InSyncMechanism)
		: SyncMechanism(&InSyncMechanism)
	{
	}

	/** FRunnable interface */
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Exit() override;
	virtual void Stop() override;
private:
	/** Hold a handle to our parent sync mechanism which handles all of our threading locks */
	class FSlateLoadingSynchronizationMechanism* SyncMechanism;
};



FSlateLoadingSynchronizationMechanism::FSlateLoadingSynchronizationMechanism(
	TSharedPtr<FMoviePlayerWidgetRenderer, ESPMode::ThreadSafe> InWidgetRenderer, 
	const TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe>& InMovieStreamer)
	: WidgetRenderer(InWidgetRenderer)
	, MovieStreamer(InMovieStreamer)
{
}

FSlateLoadingSynchronizationMechanism::~FSlateLoadingSynchronizationMechanism()
{
	DestroySlateThread();
}

void FSlateLoadingSynchronizationMechanism::Initialize()
{
	check(IsInGameThread());

	ResetSlateDrawPassEnqueued();
	SetSlateMainLoopRunning();

	bMainLoopRunning = true;

	FString ThreadName = "SlateLoadingThread";
	ThreadName.AppendInt(LoadingThreadInstanceCounter.Increment());

	SlateRunnableTask = new FSlateLoadingThreadTask( *this );
	SlateLoadingThread = FRunnableThread::Create(SlateRunnableTask, *ThreadName);
}

void FSlateLoadingSynchronizationMechanism::DestroySlateThread()
{
	check(IsInGameThread());

	if (SlateLoadingThread)
	{
		IsRunningSlateMainLoop.Reset();

		while (bMainLoopRunning)
		{
			FPlatformApplicationMisc::PumpMessages(false);

			FPlatformProcess::Sleep(0.001f);
		}

		delete SlateLoadingThread;
		delete SlateRunnableTask;
		SlateLoadingThread = nullptr;
		SlateRunnableTask = nullptr;
	}
}

bool FSlateLoadingSynchronizationMechanism::IsSlateDrawPassEnqueued()
{
	return IsSlateDrawEnqueued.GetValue() != 0;
}

void FSlateLoadingSynchronizationMechanism::SetSlateDrawPassEnqueued()
{
	IsSlateDrawEnqueued.Set(1);
}

void FSlateLoadingSynchronizationMechanism::ResetSlateDrawPassEnqueued()
{
	IsSlateDrawEnqueued.Reset();
}

bool FSlateLoadingSynchronizationMechanism::IsSlateMainLoopRunning()
{
	return IsRunningSlateMainLoop.GetValue() != 0;
}

void FSlateLoadingSynchronizationMechanism::SetSlateMainLoopRunning()
{
	IsRunningSlateMainLoop.Set(1);
}

void FSlateLoadingSynchronizationMechanism::ResetSlateMainLoopRunning()
{
	IsRunningSlateMainLoop.Reset();
}

void FSlateLoadingSynchronizationMechanism::SlateThreadInitFailed()
{
	bMainLoopRunning = false;
}

void FSlateLoadingSynchronizationMechanism::SlateThreadRunMainLoop()
{
	double LastTime = FPlatformTime::Seconds();

	while (IsSlateMainLoopRunning())
	{
		// Test to ensure that we are still the SlateLoadingThread
		checkCode(
			const int32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			const int32 PreviousThreadId = FPlatformAtomics::InterlockedCompareExchange((int32*)&GSlateLoadingThreadId, CurrentThreadId, CurrentThreadId);
			check(PreviousThreadId == CurrentThreadId)
		);

		double CurrentTime = FPlatformTime::Seconds();
		double DeltaTime = CurrentTime - LastTime;

		// 60 fps max
		const double MaxTickRate = 1.0/60.0f;

		const double TimeToWait = MaxTickRate - DeltaTime;

		if( TimeToWait > 0 )
		{
			FPlatformProcess::Sleep((float)TimeToWait);
			CurrentTime = FPlatformTime::Seconds();
			DeltaTime = CurrentTime - LastTime;
		}

		if (FSlateApplication::IsInitialized() && !IsSlateDrawPassEnqueued())
		{
			// Tick engine stuff.
			if (MovieStreamer.IsValid())
			{
				MovieStreamer->TickPreEngine();
				MovieStreamer->TickPostEngine();
			}

			FSlateRenderer* MainSlateRenderer = FSlateApplication::Get().GetRenderer();
			FScopeLock ScopeLock(MainSlateRenderer->GetResourceCriticalSection());

			WidgetRenderer->DrawWindow((float)DeltaTime);

			SetSlateDrawPassEnqueued();

			// Tick after rendering.
			if (MovieStreamer.IsValid())
			{
				MovieStreamer->TickPostRender();
			}
		}

		LastTime = CurrentTime;
	}
	
	while (IsSlateDrawPassEnqueued())
	{
		FPlatformProcess::Sleep(1.f / 60.f);
	}
	
	bMainLoopRunning = false;
}


bool FSlateLoadingThreadTask::Init()
{
	// First thing to do is set the slate loading thread ID
	// This guarantees all systems know that a slate thread exists
	const int32 PreviousValue = FPlatformAtomics::InterlockedCompareExchange((int32*)&GSlateLoadingThreadId, FPlatformTLS::GetCurrentThreadId(), 0);

	bool bSuccess = PreviousValue == 0;
	ensureMsgf(bSuccess, TEXT("Only one system can use the SlateThread at the same time. GetMoviePlayer is not compatible with PreLoadScreen."));
	if (!bSuccess)
	{
		SyncMechanism->SlateThreadInitFailed();
	}
	return bSuccess;
}

uint32 FSlateLoadingThreadTask::Run()
{
	FTaskTagScope Scope(ETaskTag::ESlateThread);
	check( GSlateLoadingThreadId == FPlatformTLS::GetCurrentThreadId() );

	SyncMechanism->SlateThreadRunMainLoop();

	return 0;
}

void FSlateLoadingThreadTask::Exit()
{
	// Tear down the slate loading thread ID
	const int32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	const int32 PreviousThreadId = FPlatformAtomics::InterlockedCompareExchange((int32*)&GSlateLoadingThreadId, 0, CurrentThreadId);
	check(PreviousThreadId == CurrentThreadId);
}

void FSlateLoadingThreadTask::Stop()
{
	SyncMechanism->ResetSlateDrawPassEnqueued();
	SyncMechanism->ResetSlateMainLoopRunning();
}
