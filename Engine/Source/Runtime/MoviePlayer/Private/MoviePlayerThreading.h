// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"

class FRunnable;
class FMoviePlayerWidgetRenderer;
class IMovieStreamer;

/**
 * This class will handle all the nasty bits about running Slate on a separate thread
 * and then trying to sync it up with the game thread and the render thread simultaneously
 */
class FSlateLoadingSynchronizationMechanism
{
public:
	FSlateLoadingSynchronizationMechanism(
		TSharedPtr<FMoviePlayerWidgetRenderer, ESPMode::ThreadSafe> InWidgetRenderer,
		const TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe>& InMovieStreamer);
	~FSlateLoadingSynchronizationMechanism();
	
	/** Sets up the locks in their proper initial state for running */
	void Initialize();

	/** Cleans up the slate thread */
	void DestroySlateThread();

	/** Handles the strict alternation of the slate drawing passes */
	bool IsSlateDrawPassEnqueued();
	void SetSlateDrawPassEnqueued();
	void ResetSlateDrawPassEnqueued();

	/** Handles the counter to determine if the slate thread should keep running */
	bool IsSlateMainLoopRunning();
	void SetSlateMainLoopRunning();
	void ResetSlateMainLoopRunning();

	/** The main loop to be run from the Slate thread */
	void SlateThreadRunMainLoop();
	/** The main loop could not be initialized */
	void SlateThreadInitFailed();

private:

	/** Used as a spin lock when we're running the primary loading loop, so that we can shutdown safely. */
	TAtomic<bool> bMainLoopRunning;

	/**
	 * This counter handles running the main loop of the slate thread
	 */
	FThreadSafeCounter IsRunningSlateMainLoop;
	/**
	 * This counter handles strict alternation between the slate thread and the render thread
	 * for passing Slate render draw passes between each other.
	 */
	FThreadSafeCounter IsSlateDrawEnqueued;

	/**
	* This counter is used to generate a unique id for each new instance of the loading thread
	*/
	static FThreadSafeCounter LoadingThreadInstanceCounter;

	/** The worker thread that will become the Slate thread */
	FRunnableThread* SlateLoadingThread;
	FRunnable* SlateRunnableTask;

	TSharedPtr<FMoviePlayerWidgetRenderer, ESPMode::ThreadSafe> WidgetRenderer;
	/** Holds the current MovieStreamer. */
	TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> MovieStreamer;
};
