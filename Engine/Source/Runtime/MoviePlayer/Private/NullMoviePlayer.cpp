// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullMoviePlayer.h"




TSharedPtr<FNullGameMoviePlayer> FNullGameMoviePlayer::MoviePlayer;
