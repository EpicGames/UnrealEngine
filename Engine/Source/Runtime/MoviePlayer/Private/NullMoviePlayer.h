// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoviePlayer.h"

/** An placeholder implementation of the movie player for the editor */
class FNullGameMoviePlayer : public IGameMoviePlayer,
	public TSharedFromThis<FNullGameMoviePlayer>
{
public:
	static void Create()
	{
		check(IsInGameThread() && !IsInSlateThread());
		check(!MoviePlayer.IsValid());

		MoviePlayer = MakeShareable(new FNullGameMoviePlayer);
	}

	static void Destroy()
	{
		check(IsInGameThread() && !IsInSlateThread());

		MoviePlayer.Reset();
	}

	static FNullGameMoviePlayer* Get()
	{
		return MoviePlayer.Get();
	}

	/** IGameMoviePlayer Interface */
	virtual void RegisterMovieStreamer(TSharedPtr<class IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer) override {}
	virtual void Initialize(class FSlateRenderer& InSlateRenderer, TSharedPtr<SWindow> RenderTarget = nullptr) override {}
	virtual void Shutdown() override {}
	virtual void PassLoadingScreenWindowBackToGame() const override {}
	virtual void SetupLoadingScreen(const FLoadingScreenAttributes& InLoadingScreenAttributes) override {}
	virtual bool HasEarlyStartupMovie() const override { return false; }
	virtual bool IsInitialized() const override { return false; }
	virtual bool PlayEarlyStartupMovies() override { return false; }
	virtual bool PlayMovie() override { return false; }
	virtual void StopMovie() override {}
	virtual void WaitForMovieToFinish(bool bAllowEngineTick = false) override {}
	virtual bool IsLoadingFinished() const override {return true;}
	virtual bool IsMovieCurrentlyPlaying() const override  {return false;}
	virtual bool LoadingScreenIsPrepared() const override {return false;}
	virtual void SetupLoadingScreenFromIni() override {}
	virtual FOnPrepareLoadingScreen& OnPrepareLoadingScreen() override { return OnPrepareLoadingScreenDelegate; }
	virtual FOnMoviePlaybackStarted& OnMoviePlaybackStarted() override { return OnMoviePlaybackStartedDelegate; }
	virtual FOnMoviePlaybackTick& OnMoviePlaybackTick() override { return OnMoviePlaybackTickDelegate; }
	virtual FOnMoviePlaybackFinished& OnMoviePlaybackFinished() override { return OnMoviePlaybackFinishedDelegate; }
	virtual FOnMovieClipFinished& OnMovieClipFinished() override { return OnMovieClipFinishedDelegate; }
	virtual void SetSlateOverlayWidget(TSharedPtr<SWidget> NewOverlayWidget) override { }
	virtual bool WillAutoCompleteWhenLoadFinishes() override { return false; }
	virtual FString GetMovieName() override { return TEXT(""); }
	virtual bool IsLastMovieInPlaylist() override { return false; }


private:
	FNullGameMoviePlayer() {}
	
private:
	/** Singleton handle */
	static TSharedPtr<FNullGameMoviePlayer> MoviePlayer;
	
	/** Called before a movie is queued up to play to configure the movie player accordingly. */
	FOnPrepareLoadingScreen OnPrepareLoadingScreenDelegate;

	FOnMoviePlaybackStarted OnMoviePlaybackStartedDelegate;
	FOnMoviePlaybackTick OnMoviePlaybackTickDelegate;
	FOnMoviePlaybackFinished OnMoviePlaybackFinishedDelegate;
	FOnMovieClipFinished OnMovieClipFinishedDelegate;

};
