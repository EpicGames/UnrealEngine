// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/SWidget.h"
#include "Slate/SlateTextures.h"

#include "MoviePlayer.generated.h"

// In order for a platform to support early movie playback, the platform must support the rendering thread 
// starting very early and support rendering as soon as it is started and the module containing the movie streamer for the platform must already be loaded

UENUM()
enum EMoviePlaybackType : int
{
	/** Normal playback mode.  Play each movie in the play list a single time */
	MT_Normal UMETA(DisplayName="Normal Playback"),
	/** Looped playback mode.  Play all movies in the play list in order then start over until manually canceled */
	MT_Looped UMETA(DisplayName = "Looped Playback"),
	/** Alternate Looped mode.  Play all of the movies in the play list and loop just the last movie until loading is finished. */
	MT_LoadingLoop UMETA(DisplayName = "Looped Last Playback"),
	MT_MAX UMETA(Hidden)
};

/** This viewport is a simple interface for the loading to use to display the video textures. */
class FMovieViewport : public ISlateViewport, public TSharedFromThis<FMovieViewport>
{
public:
	FMovieViewport() : DefaultSize(ForceInitToZero) {}
	~FMovieViewport() = default;

	/* ISlateViewport interface. */
	virtual FIntPoint GetSize() const override
	{
		TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> SlateTextureSharedPtr = SlateTexture.Pin();
		return SlateTextureSharedPtr.IsValid() ? FIntPoint(SlateTextureSharedPtr->GetWidth(), SlateTextureSharedPtr->GetHeight()) : DefaultSize;
	}

	virtual class FSlateShaderResource* GetViewportRenderTargetTexture() const override
	{
		return SlateTexture.IsValid() ? SlateTexture.Pin().Get() : NULL;
	}

	virtual bool RequiresVsync() const override
	{
		return false;
	}

	void SetTexture(TWeakPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> InTexture)
	{
		SlateTexture = InTexture;
	}

	/** Sets the size to use when there is no texture. */
	void SetDefaultSize(FIntPoint InSize)
	{
		DefaultSize = InSize;
	}

private:
	TWeakPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> SlateTexture;

	/** Size to use when we don't have a texture. */
	FIntPoint DefaultSize;
};


/** Interface for creating a movie streaming player. Should be one instance per platform. */
class IMovieStreamer
{
public:
	/**
	 * Initializes this movie streamer with all the movie paths (ordered) we want to play
	 * Movie paths are local to the current game's Content/Movies/ directory.
	 */

	virtual bool Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType) = 0;
	
	/** Tells the movie streamer about the viewport interface that was active before us. */
	virtual void PreviousViewportInterface(const TSharedPtr<ISlateViewport>& PreviousViewportInterface) {};

	/** Forces the movie streamer to cancel what it's streaming and close. */
	virtual void ForceCompletion() = 0;

	/** Code run every tick for any additional per tick handling of playing the movie. Returns true if done. */
	virtual bool Tick(FRHICommandListBase& RHICmdList, float DeltaTime) = 0;

	/** Gets a viewport interface which will be used to draw the movie. */
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() = 0;

	/** Gets the aspect ratio of the movie frames being streamed. */
	virtual float GetAspectRatio() const = 0;

	/** returns the name of the movie currently being played */
	virtual FString GetMovieName() = 0;

	/** returns true if the movie being played in the last one in the play list */
	virtual bool IsLastMovieInPlaylist() = 0;

	/** Called to allow the movie streamer to cleanup any resources once there are no movies left to play. */
	virtual void Cleanup() = 0;

	virtual FTextureRHIRef GetTexture() { return nullptr; }

	virtual ~IMovieStreamer() = default;

	DECLARE_EVENT_OneParam(IMovieStreamer, FOnCurrentMovieClipFinished, const FString&)
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() = 0;

	void BroadcastCurrentMovieClipFinished(const FString& MovieClipThatFinished) { OnCurrentMovieClipFinished().Broadcast(MovieClipThatFinished); }

	/** Called by the engine on suspend/resume */
	virtual void Suspend() {};
	virtual void Resume() {};

	// MediaFramework support.
	/** Called before the engine tick. */
	virtual void TickPreEngine() {};
	/** Called after the engine ticks. */
	virtual void TickPostEngine() {};
	/** Called after rendering is finished. */
	virtual void TickPostRender() {};
};



/** Struct of all the attributes a loading screen will have. */
struct FLoadingScreenAttributes
{
	FLoadingScreenAttributes()
		: MinimumLoadingScreenDisplayTime(-1.0f)
		, bAutoCompleteWhenLoadingCompletes(true)
		, bMoviesAreSkippable(true)
		, bWaitForManualStop(false)
		, bAllowInEarlyStartup(false)
		, bAllowEngineTick(false)
		, PlaybackType(EMoviePlaybackType::MT_Normal) {}

	/** The widget to be displayed on top of the movie or simply standalone if there is no movie. */
	TSharedPtr<class SWidget> WidgetLoadingScreen;

	/** The movie paths local to the game's Content/Movies/ directory we will play. */
	TArray<FString> MoviePaths;

	/** The minimum time that a loading screen should be opened for. */
	float MinimumLoadingScreenDisplayTime;

	/** If true, the loading screen will disappear as soon as all movies are played and loading is done. */
	bool bAutoCompleteWhenLoadingCompletes;

	/** If true, movies can be skipped by clicking the loading screen as long as loading is done. */
	bool bMoviesAreSkippable;

	/** If true, movie playback continues until Stop is called. */
	bool bWaitForManualStop;

	/** If true loading screens here cannot have any uobjects of any kind or use any engine features at all. This will start the movies very early as a result on platforms that support it */
	bool bAllowInEarlyStartup;

	/** If true, this will call the engine tick while the game thread is stalled waiting for a loading movie to finish. This only works for post-startup load screens and is potentially unsafe */
	bool bAllowEngineTick;

	/** Should we just play back, loop, etc.  NOTE: if the playback type is MT_LoopLast, then bAutoCompleteWhenLoadingCompletes will be togged on when the last movie is hit*/
	TEnumAsByte<EMoviePlaybackType> PlaybackType;

	/** True if there is either a standalone widget or any movie paths or both. */
	MOVIEPLAYER_API bool IsValid() const;

	/** Creates a simple test loading screen widget. */
	static MOVIEPLAYER_API TSharedRef<class SWidget> NewTestLoadingScreenWidget();
};


/** An interface to the movie player we will use for loading screens and gameplay movies */
class IGameMoviePlayer
{
public:
	/** Registers a movie streamer with the movie player. Set in the preloading screen stage. */
	virtual void RegisterMovieStreamer(TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer) = 0;

	/** Initializes this movie player, creating the startup window and hiding the splash screen. To be called in the launch engine loop. */
	virtual void Initialize(class FSlateRenderer& InSlateRenderer, TSharedPtr<SWindow> RenderTarget = nullptr) = 0;

	/** Shutsdown the movie player. */
	virtual void Shutdown() = 0;

	/** Passes the loading screen window back to the game to use. For use by the launch engine loop only. */
	virtual void PassLoadingScreenWindowBackToGame() const = 0;
	
	/** Passes in a slate loading screen UI, movie paths, and any additional data. */
	virtual void SetupLoadingScreen(const FLoadingScreenAttributes& InLoadingScreenAttributes) = 0;
	
	/** Called before playing a movie if the loading screen has not been prepared. */
	DECLARE_EVENT(IGameMoviePlayer, FOnPrepareLoadingScreen)
	virtual FOnPrepareLoadingScreen& OnPrepareLoadingScreen() = 0;

	/**
	 * @return true if the movie player has been set up with an early startup movie
	 */
	virtual bool HasEarlyStartupMovie() const = 0;

	/**
	 * Play any early start up movies that have been set up
	 *
	 * @return true if a movie started playing
	 */
	virtual bool PlayEarlyStartupMovies() = 0;

	/**
	 * @return true if Initialize was called
	 */
	virtual bool IsInitialized() const = 0;

	/** 
	 * Starts playing the movie given the last FLoadingScreenAttributes passed in
	 * @return true of a movie started playing.
	 */
	virtual bool PlayMovie() = 0;

	/** 
	 * Stops the currently playing movie, if any.
	 */
	virtual void StopMovie() = 0;
	
	/** Call only on the game thread. Spins this thread until the movie stops. If bAllowEngineTick is true, the engine tick may run inside here */
	virtual void WaitForMovieToFinish(bool bAllowEngineTick = false) = 0;

	/** Called from to check if the game thread is finished loading. */
	virtual bool IsLoadingFinished() const = 0;

	/** True if the loading screen is currently running (i.e. PlayMovie but no WaitForMovieToFinish has been called). */
	virtual bool IsMovieCurrentlyPlaying() const = 0;

	/** True if we have either slate widgets or a movie to show. */
	virtual bool LoadingScreenIsPrepared() const = 0;
	
	/** Sets up an FLoadingScreenAttributes from the game's engine.ini, then calls the virtual SetupLoadingScreen. */
	virtual void SetupLoadingScreenFromIni() = 0;

	/** returns the name of the movie currently being played */
	virtual FString GetMovieName() = 0;

	/** returns true if the movie being played in the last one in the play list */
	virtual bool IsLastMovieInPlaylist() = 0;

	/* Callback for when the LoadingScreen setup above in WidgetLoadingScreen is displayed **/
	DECLARE_EVENT(IGameMoviePlayer, FOnMoviePlaybackStarted)
	virtual FOnMoviePlaybackStarted& OnMoviePlaybackStarted() = 0;

	/** Callback for when the game thread is blocked but you want to do some ticking. */
	DECLARE_EVENT_OneParam(IGameMoviePlayer, FOnMoviePlaybackTick, float DeltaTime)
	virtual FOnMoviePlaybackTick& OnMoviePlaybackTick() = 0;

	DECLARE_EVENT(IGameMoviePlayer, FOnMoviePlaybackFinished)
	virtual FOnMoviePlaybackFinished& OnMoviePlaybackFinished() = 0;

	DECLARE_EVENT_OneParam(IGameMoviePlayer, FOnMovieClipFinished, const FString&)
	virtual FOnMovieClipFinished& OnMovieClipFinished() = 0;

	/** Allows for a slate overlay widget to be set after playback. */
	virtual void SetSlateOverlayWidget(TSharedPtr<SWidget> NewOverlayWidget) = 0;
	/** Allows for a DPI scale to be set. */
	virtual void SetViewportDPIScale(float InViewportDPIScale) {}

	void BroadcastMoviePlaybackFinished() { OnMoviePlaybackFinished().Broadcast(); }
	void BroadcastMovieClipFinished(const FString& MovieClipThatFinished) { OnMovieClipFinished().Broadcast(MovieClipThatFinished); }

	/** This function shouild return true if the movie will auto-complete the sequence when background loading has finished */
	virtual bool WillAutoCompleteWhenLoadFinishes() = 0;

	virtual ~IGameMoviePlayer() = default;

	/** Check if the initial movie(s) is still playing */
	virtual bool IsStartupMoviePlaying() const { return false; };
	virtual void ForceCompletion() {};
	virtual void Suspend() {};
	virtual void Resume() {};

	/** Call this to have the MoviePlayer play in blocking sections and not loadmap. */
	virtual void SetIsPlayOnBlockingEnabled(bool bIsEnabled) {}
};

/** Creates the movie player */
MOVIEPLAYER_API void CreateMoviePlayer();

/** Destroys the movie player */
MOVIEPLAYER_API void DestroyMoviePlayer();

/** Gets the movie player singleton for the engine. */
MOVIEPLAYER_API IGameMoviePlayer* GetMoviePlayer();

MOVIEPLAYER_API IGameMoviePlayer& GetMoviePlayerRef();

/** Returns true if the movie player is enabled. */
bool MOVIEPLAYER_API IsMoviePlayerEnabled();
