// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MoviePlayerSettings.generated.h"


/**
 * Implements the settings for the Windows target platform.
 */
UCLASS(config=Game, defaultconfig, MinimalAPI)
class UMoviePlayerSettings
	: public UObject
{
public:

	GENERATED_UCLASS_BODY()

	/** If enabled, The game waits for startup movies to complete even if loading has finished. */
	UPROPERTY(globalconfig, EditAnywhere, Category="Movies")
	bool bWaitForMoviesToComplete;

	/** If enabled, Startup movies can be skipped by the user when a mouse button is pressed. */
	UPROPERTY(globalconfig, EditAnywhere, Category="Movies")
	bool bMoviesAreSkippable;

	/** Movies to play on startup. Note that these must be in your game's Game/Content/Movies directory. */
	UPROPERTY(globalconfig, EditAnywhere, Category="Movies")
	TArray<FString> StartupMovies;
};
